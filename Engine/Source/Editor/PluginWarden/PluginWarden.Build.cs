// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PluginWarden : ModuleRules
{
	public PluginWarden(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Analytics",
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
				"PortalServices",
				"LauncherPlatform",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"LauncherServices",
			}
		);
	}
}
