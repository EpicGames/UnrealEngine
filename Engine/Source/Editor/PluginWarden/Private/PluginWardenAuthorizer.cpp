// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginWardenAuthorizer.h"
#include "PluginWardenModule.h"

#include "Containers/Ticker.h"
#include "Async/TaskGraphInterfaces.h"
#include "Editor.h"

#include "Logging/LogMacros.h"

#include "IPortalServiceLocator.h"
#include "Account/IPortalUserLogin.h"
#include "Application/IPortalApplicationWindow.h"

#include "ILauncherPlatform.h"
#include "LauncherPlatformModule.h"

extern TSet<FString> AuthorizedPlugins;

DEFINE_LOG_CATEGORY(PluginWarden);

enum
{
	MaxAuthorizationRetries = 3,	// Max number of authorization check to retry when the entitlement wasn't retrieved yet
	MaxSigninRetries = 3,			// Max number of prompt user for sign-in to retry when the Launcher fails to handle the prompt request
	MaxLauncherRetries = 3,			// Max number of Launcher start to retry when it's detected that it's not available for user sign-in status check
	GeneralWaitingTimeout = 15,		// Timeout in secs to use for various waiting phases
	SigninWaitingTimeout = 120,		// Timeout in secs to use during the Waiting for sign-in phase
	UserDetailsCheckPeriod = 10		// The periodicity in secs to check for the user details during the Waiting for sign-in phase
};	

#define LOCTEXT_NAMESPACE "PluginWardenAuthorization"

FPluginWardenAuthorizer::FPluginWardenAuthorizer(const FText& InPluginFriendlyName, const FString& InPluginItemId, const FString& InPluginOfferId, const EEntitlementCacheLevelRequest InCacheLevel)
	: CurrentState(EPluginAuthorizationState::Initializing)
	, CacheLevel(InCacheLevel)
	, WaitingTime(0)
	, PluginFriendlyName(InPluginFriendlyName)
	, PluginItemId(InPluginItemId)
	, PluginOfferId(InPluginOfferId)
	, NumAuthorizationRetries(0)
	, NumSignInRetries(0)
	, NumLauncherRetries(0)
	, CurrentWaitLoopNumber(0)
{
	TSharedRef<IPortalServiceLocator> ServiceLocator = GEditor->GetServiceLocator();
	PortalWindowService = ServiceLocator->GetServiceRef<IPortalApplicationWindow>();
	PortalUserService = ServiceLocator->GetServiceRef<IPortalUser>();
	PortalUserLoginService = ServiceLocator->GetServiceRef<IPortalUserLogin>();
}

EPluginAuthorizationState FPluginWardenAuthorizer::UpdateAuthorizationState(float DeltaTime)
{
	EPluginAuthorizationState NewState = CurrentState;

	switch ( CurrentState )
	{
		case EPluginAuthorizationState::Initializing:
		{
			WaitingTime = 0;
			if ( PortalWindowService->IsAvailable() && PortalUserService->IsAvailable() )
			{
				NewState = EPluginAuthorizationState::AuthorizePlugin;
			}
			else
			{
				NewState = EPluginAuthorizationState::StartLauncher;
			}
			break;
		}
		case EPluginAuthorizationState::StartLauncher:
		{
			WaitingTime = 0;
			ILauncherPlatform* LauncherPlatform = FLauncherPlatformModule::Get();

			if (LauncherPlatform != nullptr )
			{
				if ( !FPlatformProcess::IsApplicationRunning(TEXT("EpicGamesLauncher")) &&
						!FPlatformProcess::IsApplicationRunning(TEXT("EpicGamesLauncher-Mac-Shipping")) )
				{
					FOpenLauncherOptions SilentOpen;
					if (LauncherPlatform->OpenLauncher(SilentOpen) )
					{
						NewState = EPluginAuthorizationState::StartLauncher_Waiting;
					}
					else
					{
						NewState = EPluginAuthorizationState::LauncherStartFailed;
					}
				}
				else
				{
					// If the process is found to be running already, move into the next state.
					NewState = EPluginAuthorizationState::StartLauncher_Waiting;
				}
			}
			else
			{
				NewState = EPluginAuthorizationState::LauncherStartFailed;
			}
			break;
		}
		case EPluginAuthorizationState::StartLauncher_Waiting:
		{
			UE_LOGF(PluginWarden, Log, "Waiting for launcher to run for the past %f seconds", WaitingTime);
			if ( FPlatformProcess::IsApplicationRunning(TEXT("EpicGamesLauncher")) && PortalWindowService->IsAvailable() && PortalUserService->IsAvailable() )
			{
				NewState = EPluginAuthorizationState::AuthorizePlugin;
			}
			else
			{
				WaitingTime += DeltaTime;
			}
			break;
		}
		case EPluginAuthorizationState::AuthorizePlugin:
		{
			WaitingTime = 0;
			EntitlementResult = PortalUserService->IsEntitledToItem(PluginItemId, CacheLevel);
			NewState = EPluginAuthorizationState::AuthorizePlugin_Waiting;
			break;
		}
		case EPluginAuthorizationState::AuthorizePlugin_Waiting:
		{
			WaitingTime += DeltaTime;

			check(EntitlementResult.GetFuture().IsValid());
			if ( EntitlementResult.GetFuture().IsReady() )
			{
				FPortalUserIsEntitledToItemResult Entitlement = EntitlementResult.GetFuture().Get();
				if ( Entitlement.IsEntitled )
				{
					NewState = EPluginAuthorizationState::Authorized;
				}
				else
				{
					NewState = EPluginAuthorizationState::IsUserSignedIn;
				}
			}

			break;
		}
		case EPluginAuthorizationState::IsUserSignedIn:
		{
			WaitingTime = 0;
			UserDetailsResult = PortalUserService->GetUserDetails();
			NewState = EPluginAuthorizationState::IsUserSignedIn_Waiting;
			break;
		}
		case EPluginAuthorizationState::IsUserSignedIn_Waiting:
		{
			WaitingTime += DeltaTime;

			check(UserDetailsResult.GetFuture().IsValid());
			if ( UserDetailsResult.GetFuture().IsReady() )
			{
				FPortalUserDetails UserDetails = UserDetailsResult.GetFuture().Get();

				if ( UserDetails.IsSignedIn )
				{
					FPortalUserIsEntitledToItemResult Entitlement = EntitlementResult.GetFuture().Get();
					if ( Entitlement.RetrievedFromCacheLevel == EEntitlementCacheLevelRetrieved::None )
					{
						// This is the case where there is no cached entitlement data and the latest entitlements have yet to be
						// retrieved so we still don't know if the user is entitled so try again
						if ( ++NumAuthorizationRetries < MaxAuthorizationRetries )
						{
							NewState = EPluginAuthorizationState::AuthorizePlugin;
						}
						else
						{
							// Give up and assume the user wasn't entitled
							NewState = EPluginAuthorizationState::Unauthorized;
						}
					}
					else
					{
						// The entitlement check was valid, the user is signed in and is not entitled, so clearly Unauthorized
						NewState = EPluginAuthorizationState::Unauthorized;
					}
				}
				else
				{
					// If they're not signed in, but they were unauthorized, they may have purchased it
					// they may just need to sign-in.
					if ( PortalUserLoginService->IsAvailable() )
					{
						NewState = EPluginAuthorizationState::SigninRequired;
					}
					else if ( ++NumLauncherRetries < MaxLauncherRetries )
					{
						// During testing, there's a flow where it goes through StartLauncher -> StartLauncher_Waiting -> AuthorizePlugin and
						// ends up here without the Launcher actually running so try starting the Launcher again
						NewState = EPluginAuthorizationState::StartLauncher;
					}
					else
					{
						// Give up and tell the user to sign in manually
						NewState = EPluginAuthorizationState::SigninFailed;
					}
				}
			}

			break;
		}
		case EPluginAuthorizationState::SigninRequired:
		{
			WaitingTime = 0;
			UserSigninResult = PortalUserLoginService->PromptUserForSignIn();
			NewState = EPluginAuthorizationState::SigninRequired_Waiting;
			break;
		}
		case EPluginAuthorizationState::SigninRequired_Waiting:
		{
			// We don't advance the wait time in the sign-required state, as this may take a long time.

			check(UserSigninResult.GetFuture().IsValid());
			if ( UserSigninResult.GetFuture().IsReady() )
			{
				// Note that the result of PromptUserForSignIn only says whether the portal successfully received and handled the request.
				// It doesn't mean that the user signed in successfully or even that the existing user signed out.
				bool IsUserPromptHandled = UserSigninResult.GetFuture().Get();
				if ( IsUserPromptHandled )
				{
					// In this case, we only know that the user prompt was successful but we assume that the user signed in and that we can retrieve the details
					UserDetailsResult = PortalUserService->GetUserDetails();
					NewState = EPluginAuthorizationState::Signin_Waiting;
					WaitingTime = 0;
				}
				else
				{
					// This state doesn't mean that the user is unauthorized but that the portal user login service wasn't able to handle the request
					// So try again
					if ( ++NumSignInRetries < MaxSigninRetries )
					{
						if (PortalUserLoginService->IsAvailable())
						{
							NewState = EPluginAuthorizationState::SigninRequired;
						}
						else
						{
							NewState = EPluginAuthorizationState::StartLauncher;
						}
					}
					else
					{
						// Give up and tell the user to sign in manually
						NewState = EPluginAuthorizationState::SigninFailed;
					}
				}
			}

			break;
		}
		// We stay in the Signin_Waiting state until the user is signed in or the timeout is reached.
		case EPluginAuthorizationState::Signin_Waiting:
		{
			WaitingTime += DeltaTime;
			UE_LOGF(PluginWarden, Log, "Waiting for sign in for the past %f seconds", WaitingTime);

			check(UserDetailsResult.GetFuture().IsValid());
			if ( UserDetailsResult.GetFuture().IsReady() )
			{
				FPortalUserDetails UserDetails = UserDetailsResult.GetFuture().Get();
				int CurrentWaitSec = ((int)WaitingTime);

				if ( UserDetails.IsSignedIn )
				{
					// If the user is now signed in, we can check for authorization again
					NewState = EPluginAuthorizationState::AuthorizePlugin;
				}
				else if ( ( CurrentWaitSec % UserDetailsCheckPeriod ) == 0 && CurrentWaitSec != CurrentWaitLoopNumber )
				{
					// Every check period, try getting the UserDetails once to see if there's been any update in the signed-in status
					UserDetailsResult = PortalUserService->GetUserDetails();
				}
			}

			break;
		}
	}

	// If we're in a waiting state, check to see if we're over the timeout period.
	switch ( NewState )
	{
		case EPluginAuthorizationState::StartLauncher_Waiting:
		case EPluginAuthorizationState::AuthorizePlugin_Waiting:
		case EPluginAuthorizationState::IsUserSignedIn_Waiting:
		case EPluginAuthorizationState::SigninRequired_Waiting:
		{
			if ( WaitingTime > (float)GeneralWaitingTimeout)
			{
				NewState = EPluginAuthorizationState::Timeout;
			}
			break;
		}
		case EPluginAuthorizationState::Signin_Waiting:
		{
			if ( WaitingTime > (float)SigninWaitingTimeout )
			{
				NewState = EPluginAuthorizationState::SigninFailed;
			}
		}
		break;
	}

	CurrentState = NewState;

	return CurrentState;
}

#undef LOCTEXT_NAMESPACE
