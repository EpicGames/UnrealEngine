// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/CoreMisc.h"
#include "Containers/Ticker.h"
#include "Async/AsyncResult.h"
#include "Account/IPortalUser.h"

class IPortalApplicationWindow;
class IPortalUserLogin;

enum class EPluginAuthorizationState
{
	Initializing,
	StartLauncher,
	StartLauncher_Waiting,
	AuthorizePlugin,
	AuthorizePlugin_Waiting,
	IsUserSignedIn,
	IsUserSignedIn_Waiting,
	SigninRequired,
	SigninRequired_Waiting,
	Signin_Waiting,
	Signin_Timeout,
	SigninFailed,
	Authorized,
	Unauthorized,
	LauncherStartFailed,
	Timeout,
	Canceled,
};

class FPluginWardenAuthorizer
{
public:
	FPluginWardenAuthorizer(const FText& InPluginFriendlyName, const FString& InPluginItemId, const FString& InPluginOfferId, const EEntitlementCacheLevelRequest InCacheLevel);

	/** Returns new state after elapsed time */
	EPluginAuthorizationState UpdateAuthorizationState(float DeltaTime);

	const FText& GetPluginFriendlyName() { return PluginFriendlyName; }
	const FString& GetPluginItemId() { return PluginItemId; }
	const FString& GetPluginOfferId() { return PluginOfferId; }

private:
	/** The current state of the plug-in authorization pipeline. */
	EPluginAuthorizationState CurrentState;

	/** Cache level to check for the entitlement */
	EEntitlementCacheLevelRequest CacheLevel;

	/** The amount of time we've been waiting for confirmation for a given step.  It's possible a problem may arise and we need to timeout. */
	float WaitingTime;

	/** The display name of the plug-in used in the auto generated dialog text. */
	FText PluginFriendlyName;

	/** The unique id of the item for the plug-in on the marketplace. */
	FString PluginItemId;

	/** The unique id of the offer for the plug-in on the marketplace. */
	FString PluginOfferId;

	/** The current number of authorization check retries. */
	unsigned int NumAuthorizationRetries;

	/** The current number of sign-in retries. */
	unsigned int NumSignInRetries;

	/** The current number of Launcher start retries. */
	unsigned int NumLauncherRetries;

	/** The current loop number of wait loops during the sign-in wait phase. */
	unsigned int CurrentWaitLoopNumber;

	/** The portal application communication service. */
	TSharedPtr<IPortalApplicationWindow> PortalWindowService;

	/** The portal user service, to allow us to check entitlements for plugins. */
	TSharedPtr<IPortalUser> PortalUserService;

	/** The portal user login service, to allow us to trigger a prompt to sign-in, if required. */
	TSharedPtr<IPortalUserLogin> PortalUserLoginService;

	/** The entitlement result we may be waiting on. */
	TAsyncResult<FPortalUserIsEntitledToItemResult> EntitlementResult;

	/** The result from the request for user details. */
	TAsyncResult<FPortalUserDetails> UserDetailsResult;

	/** The result from requesting the user sign-in, they may sign-in, they may cancel. */
	TAsyncResult<bool> UserSigninResult;
};
