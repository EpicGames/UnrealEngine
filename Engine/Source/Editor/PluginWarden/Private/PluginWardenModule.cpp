// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginWardenModule.h"

#include "AnalyticsEventAttribute.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/Ticker.h"
#include "CoreGlobals.h"
#include "EngineAnalytics.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "IAnalyticsProviderET.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PluginWardenAuthorizer.h"
#include "SAuthorizingPlugin.h"
#include "Templates/SharedPointer.h"
#include "Trace/Detail/Channel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"

IMPLEMENT_MODULE( FPluginWardenModule, PluginWarden );

#define LOCTEXT_NAMESPACE "PluginWarden"

TSet<FString> AuthorizedPlugins;

void FPluginWardenModule::StartupModule()
{

}

void FPluginWardenModule::ShutdownModule()
{

}

void FPluginWardenModule::CheckEntitlementForPlugin(const FText& PluginFriendlyName, const FString& PluginItemId, const FString& PluginOfferId, const EEntitlementCacheLevelRequest CacheLevel,
	const FText& UnauthorizedMessageOverride, EUnauthorizedErrorHandling UnauthorizedErrorHandling, TFunction<void()> AuthorizedCallback)
{
	// If we've previously authorized the plug-in, just immediately verify access.
	if ( AuthorizedPlugins.Contains(PluginItemId) )
	{
		AuthorizedCallback();
		return;
	}

	if (IsRunningCommandlet() || GIsRunningUnattendedScript)
	{
		if (RunAuthorizationPipeline(PluginFriendlyName, PluginItemId, PluginOfferId, CacheLevel))
		{
			AuthorizedPlugins.Add(PluginItemId);
			AuthorizedCallback();
		}
	}
	else
	{
		// Create the window
		TSharedRef<SWindow> AuthorizingPluginWindow = SNew(SWindow)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.HasCloseButton(true)
			.SizingRule(ESizingRule::Autosized)
			.Title(FText::Format(LOCTEXT("EntitlementCheckFormat", "{0} - Entitlement Check"), PluginFriendlyName));

		TSharedRef<SAuthorizingPlugin> PluginAuthPanel = SNew(SAuthorizingPlugin, AuthorizingPluginWindow, SAuthorizingPlugin::FPluginInfo( PluginFriendlyName, PluginItemId, PluginOfferId ), CacheLevel, AuthorizedCallback);
		PluginAuthPanel->SetUnauthorizedOverride(UnauthorizedMessageOverride, UnauthorizedErrorHandling);

		AuthorizingPluginWindow->SetContent(PluginAuthPanel);

		FSlateApplication::Get().AddModalWindow(AuthorizingPluginWindow, nullptr);
	}
}

void FPluginWardenModule::CheckEntitlementForPlugin(const FText& PluginFriendlyName, const FString& PluginItemId, const FString& PluginOfferId, const FText& UnauthorizedMessageOverride, EUnauthorizedErrorHandling UnauthorizedErrorHandling, TFunction<void()> AuthorizedCallback)
{
	CheckEntitlementForPlugin(PluginFriendlyName, PluginItemId, PluginOfferId, EEntitlementCacheLevelRequest::Memory, UnauthorizedMessageOverride, UnauthorizedErrorHandling, AuthorizedCallback);
}

bool FPluginWardenModule::RunAuthorizationPipeline(const FText& PluginFriendlyName, const FString& PluginItemId, const FString& PluginOfferId, const EEntitlementCacheLevelRequest CacheLevel)
{
	FPluginWardenAuthorizer Authorizer(PluginFriendlyName, PluginItemId, PluginOfferId, CacheLevel);

	EPluginAuthorizationState AuthorizationState = EPluginAuthorizationState::Initializing;
	EPluginAuthorizationState PreviousState = AuthorizationState;

	bool bAuthorizationCompleted = false;

	double LastLoopTime = FPlatformTime::Seconds();
	double LastTickTime = FPlatformTime::Seconds();
	const float MinThrottlePeriod = (1.0f / 60.0f); //Throttle the loop to a maximum of 60Hz

	while (!bAuthorizationCompleted)
	{
		//Throttle the loop
		const double CurrentLoopTime = FPlatformTime::Seconds();
		const float SleepTime = MinThrottlePeriod - (float)(CurrentLoopTime-LastLoopTime);
		LastLoopTime = CurrentLoopTime;
		if (SleepTime > 0.0f)
		{
			// Sleep a bit to not eat up all CPU time
			FPlatformProcess::Sleep(SleepTime);
		}

		const double CurrentTickTime = FPlatformTime::Seconds();
		float DeltaTime = (float)(CurrentTickTime - LastTickTime);
		LastTickTime = CurrentTickTime;

		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
		FTSTicker::GetCoreTicker().Tick(DeltaTime);

		PreviousState = AuthorizationState;
		AuthorizationState = Authorizer.UpdateAuthorizationState(DeltaTime);

		switch ( AuthorizationState )
		{
			case EPluginAuthorizationState::Canceled:
			case EPluginAuthorizationState::Authorized:
			case EPluginAuthorizationState::Unauthorized:
			case EPluginAuthorizationState::Timeout:
			case EPluginAuthorizationState::LauncherStartFailed:
			case EPluginAuthorizationState::SigninFailed:
			{
				bAuthorizationCompleted = true;
				break;
			}
		}

		if (PreviousState != AuthorizationState)
		{
			switch ( AuthorizationState )
			{
				case EPluginAuthorizationState::StartLauncher_Waiting:
				{
					UE_LOGF(PluginWarden, Log, "Waiting for launcher ...");
					break;
				}
				case EPluginAuthorizationState::SigninRequired_Waiting:
				{
					UE_LOGF(PluginWarden, Log, "Sign-in required ...");
					break;
				}
				case EPluginAuthorizationState::Signin_Waiting:
				{
					UE_LOGF(PluginWarden, Log, "Signing in ...");
					break;
				}
				case EPluginAuthorizationState::AuthorizePlugin_Waiting:
				{
					UE_LOGF(PluginWarden, Log, "Waiting for authorization on plug-in ...");
					break;
				}
			}
		}
	}

	switch (AuthorizationState)
	{
		case EPluginAuthorizationState::Authorized:
		{
			return true;
		}
		case EPluginAuthorizationState::Unauthorized:
		{
			UE_LOGF(PluginWarden, Warning, "It looks like your Epic Games account doesn't have entitlements for %ls.", *PluginFriendlyName.ToString());
			break;
		}
		case EPluginAuthorizationState::Timeout:
		{
			UE_LOGF(PluginWarden, Error, "We were unable to verify your access to the plugin before timing out.");
			break;
		}
		case EPluginAuthorizationState::LauncherStartFailed:
		{
			UE_LOGF(PluginWarden, Error, "Cannot start the launcher. Please open the launcher and sign in manually.");
			break;
		}
		case EPluginAuthorizationState::SigninFailed:
		{
			UE_LOGF(PluginWarden, Error, "Sign-in failed. Please sign in manually through the launcher.");
			break;
		}
	}

	TArray< FAnalyticsEventAttribute > EventAttributes;
	EventAttributes.Emplace( TEXT("State"), (int32)AuthorizationState );
	EventAttributes.Emplace( TEXT("PreviousState"), (int32)PreviousState );
	EventAttributes.Emplace( TEXT("UnauthorizedErrorHandling"), (int32)IPluginWardenModule::EUnauthorizedErrorHandling::Silent );
	EventAttributes.Emplace( TEXT("ItemId"), PluginItemId );
	EventAttributes.Emplace( TEXT("OfferId"), PluginOfferId );

	FEngineAnalytics::GetProvider().RecordEvent( TEXT("PluginWarden.AuthorizationFailure"), EventAttributes );

	return false;
}

#undef LOCTEXT_NAMESPACE
