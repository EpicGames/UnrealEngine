// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Account/IPortalUser.h"
#include "Containers/UnrealString.h"
#include "IPluginWardenModule.h"
#include "Logging/LogMacros.h"
#include "Templates/Function.h"

class FText;

DECLARE_LOG_CATEGORY_EXTERN(PluginWarden, Log, All);

/**
 * The Plugin Warden is a simple module used to verify a user has purchased a plug-in.  This
 * module won't prevent a determined user from avoiding paying for a plug-in, it is merely to
 * prevent accidental violation of a per-seat license on a plug-in, and to direct those users
 * to the marketplace page where they may purchase the plug-in.
 */
class FPluginWardenModule : public IPluginWardenModule
{
public:

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule();

	/**
	 * Ask the launcher if the user has authorization to use the given plug-in. The authorized
	 * callback will only be called if the user is authorized to use the plug-in.
	 *
	 * FPluginWardenModule& PluginWarden = FModuleManager::LoadModuleChecked<FPluginWardenModule>("PluginWarden");
	 * PluginWarden.CheckEntitlementForPlugin(LOCTEXT("AwesomePluginName", "My Awesome Plugin"), TEXT("PLUGIN_MARKETPLACE_GUID"), [&] () {
	 *		// Authorized Code Here
	 * });
	 *
	 * @param PluginFriendlyName The localized friendly name of the plug-in.
	 * @param PluginItemId The unique identifier of the item plug-in on the marketplace.
	 * @param PluginOfferId The unique identifier of the offer for the plug-in on the marketplace.
	 * @param CacheLevel Where to check for the entitlements. Defaults to memory.
	 * @param UnauthorizedMessageOverride The error message to display for unauthorized plugins, overriding the default message if not empty.
	 * @param UnauthorizedErrorHandling How to handle the unauthorized error.
	 * @param AuthorizedCallback This function will be called after the user has been given entitlement.
	 */
	virtual void CheckEntitlementForPlugin(const FText& PluginFriendlyName, const FString& PluginItemId, const FString& PluginOfferId, const EEntitlementCacheLevelRequest CacheLevel,
		const FText& UnauthorizedMessageOverride, EUnauthorizedErrorHandling UnauthorizedErrorHandling, TFunction<void()> AuthorizedCallback) override;

	virtual void CheckEntitlementForPlugin(const FText& PluginFriendlyName, const FString& PluginItemId, const FString& PluginOfferId,
		const FText& UnauthorizedMessageOverride, EUnauthorizedErrorHandling UnauthorizedErrorHandling, TFunction<void()> AuthorizedCallback) override;

private:
	bool RunAuthorizationPipeline(const FText& PluginFriendlyName, const FString& PluginItemId, const FString& PluginOfferId, const EEntitlementCacheLevelRequest CacheLevel);
};
