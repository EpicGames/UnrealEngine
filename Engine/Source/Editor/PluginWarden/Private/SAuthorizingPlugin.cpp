// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAuthorizingPlugin.h"
#include "PluginWardenAuthorizer.h"
#include "EngineAnalytics.h"
#include "Misc/MessageDialog.h"
#include "Containers/Ticker.h"
#include "Async/TaskGraphInterfaces.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "Widgets/Images/SThrobber.h"

#include "ILauncherPlatform.h"
#include "LauncherPlatformModule.h"

#define LOCTEXT_NAMESPACE "PluginWarden"

void SAuthorizingPlugin::Construct(const FArguments& InArgs, const TSharedRef<SWindow>& InParentWindow, const FPluginInfo& InPluginInfo, const EEntitlementCacheLevelRequest InCacheLevel, TFunction<void()> InAuthorizedCallback)
{
	ParentWindow = InParentWindow;
	AuthorizedCallback = InAuthorizedCallback;
	UnauthorizedErrorHandling = IPluginWardenModule::EUnauthorizedErrorHandling::ShowMessageOpenStore;

	Authorizer = MakeShared<FPluginWardenAuthorizer>(InPluginInfo.FriendlyName, InPluginInfo.ItemId, InPluginInfo.OfferId, InCacheLevel);

	InParentWindow->SetOnWindowClosed(FOnWindowClosed::CreateSP(this, &SAuthorizingPlugin::OnWindowClosed));
	bUserInterrupted = true;

	AuthorizationState = EPluginAuthorizationState::Initializing;
	PreviousAuthorizationState = AuthorizationState;

	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SAuthorizingPlugin::RefreshStatus));

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(500)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(10, 30, 10, 20)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SThrobber)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(10, 0)
					[
						SNew(STextBlock)
						.Text(this, &SAuthorizingPlugin::GetWaitingText)
						.Font(FCoreStyle::GetDefaultFontStyle("Bold", 12))
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.Padding(10)
				[
					SNew(SButton)
					.Text(LOCTEXT("CancelText", "Cancel"))
					.OnClicked(this, &SAuthorizingPlugin::OnCancel)
				]
			]
		]
	];
}

void SAuthorizingPlugin::SetUnauthorizedOverride(const FText & InUnauthorizedMessageOverride, IPluginWardenModule::EUnauthorizedErrorHandling InUnauthorizedErrorHandling)
{
	UnauthorizedMessageOverride = InUnauthorizedMessageOverride;
	UnauthorizedErrorHandling = InUnauthorizedErrorHandling;
}

FText SAuthorizingPlugin::GetWaitingText() const
{
	switch ( AuthorizationState )
	{
	case EPluginAuthorizationState::Initializing:
	case EPluginAuthorizationState::StartLauncher:
		return LOCTEXT("StartingLauncher", "Starting Epic Games Launcher...");
	case EPluginAuthorizationState::StartLauncher_Waiting:
		return LOCTEXT("ConnectingToLauncher", "Connecting...");
	case EPluginAuthorizationState::AuthorizePlugin:
	case EPluginAuthorizationState::AuthorizePlugin_Waiting:
		return FText::Format(LOCTEXT("CheckingIfYouCanUseFormat", "Checking license for {0}..."), Authorizer->GetPluginFriendlyName());
	case EPluginAuthorizationState::IsUserSignedIn:
	case EPluginAuthorizationState::IsUserSignedIn_Waiting:
		return LOCTEXT("CheckingIfUserSignedIn", "Authorization failed, checking user information...");
	case EPluginAuthorizationState::SigninRequired:
	case EPluginAuthorizationState::SigninRequired_Waiting:
		return LOCTEXT("NeedUserToLoginToCheck", "Authorization failed, sign-in required...");
	case EPluginAuthorizationState::Signin_Waiting:
		return LOCTEXT("WaitingForSignin", "Epic Games Launcher started. Waiting for you to sign in...");
	case EPluginAuthorizationState::SigninFailed:
		return LOCTEXT("SigninFailed", "Sign-in failed. Cancel and sign in manually through the launcher.");
	}

	return LOCTEXT("Processing", "Processing...");
}

EActiveTimerReturnType SAuthorizingPlugin::RefreshStatus(double InCurrentTime, float InDeltaTime)
{
	// Engine tick isn't running when the modal window is open, so we need to tick any core tickers
	// to as that's what the RPC system uses to update the current state of RPC calls.
	FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
	FTSTicker::GetCoreTicker().Tick(InDeltaTime);

	PreviousAuthorizationState = AuthorizationState;
	AuthorizationState = Authorizer->UpdateAuthorizationState(InDeltaTime);

	switch ( AuthorizationState )
	{
		case EPluginAuthorizationState::Authorized:
		case EPluginAuthorizationState::Unauthorized:
		case EPluginAuthorizationState::Timeout:
		case EPluginAuthorizationState::LauncherStartFailed:
		{
			bUserInterrupted = false;
			ParentWindow.Pin()->RequestDestroyWindow();
			break;
		}
		case EPluginAuthorizationState::Canceled:
		{
			bUserInterrupted = true;
			ParentWindow.Pin()->RequestDestroyWindow();
			break;
		}
	}

	return EActiveTimerReturnType::Continue;
}

FReply SAuthorizingPlugin::OnCancel()
{
	bUserInterrupted = true;
	ParentWindow.Pin()->RequestDestroyWindow();

	return FReply::Handled();
}

void SAuthorizingPlugin::OnWindowClosed(const TSharedRef<SWindow>& InWindow)
{
	if ( bUserInterrupted || AuthorizationState == EPluginAuthorizationState::Canceled )
	{
		// User interrupted or canceled, just close down.
	}
	else
	{
		switch ( AuthorizationState )
		{
			case EPluginAuthorizationState::Authorized:
			{
				AuthorizedPlugins.Add(Authorizer->GetPluginItemId());
				AuthorizedCallback();
				return;
			}
			case EPluginAuthorizationState::Timeout:
			{
				FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("TimeoutFailure", "Something went wrong.  We were unable to verify your access to the plugin before timing out."));
				break;
			}
			case EPluginAuthorizationState::LauncherStartFailed:
			{
				FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("LauncherStartFailure", "Something went wrong starting the launcher.  We were unable to verify your access to the plugin."));
				break;
			}
			case EPluginAuthorizationState::Unauthorized:
			{
				FText FailureMessage = UnauthorizedMessageOverride;
				if (FailureMessage.IsEmpty())
				{
					FailureMessage = FText::Format(LOCTEXT("UnauthorizedFailure", "It doesn't look like you've purchased {0}.\n\nWould you like to see the store page?"), Authorizer->GetPluginFriendlyName());
				}

				switch (UnauthorizedErrorHandling)
				{
					case IPluginWardenModule::EUnauthorizedErrorHandling::ShowMessageOpenStore:
					{
						EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNo, FailureMessage);
						if (Response == EAppReturnType::Yes)
						{
							ShowStorePageForPlugin();
						}
						break;
					}
					case IPluginWardenModule::EUnauthorizedErrorHandling::ShowMessage:
					{
						FMessageDialog::Open(EAppMsgType::Ok, FailureMessage);
						break;
					}
					case IPluginWardenModule::EUnauthorizedErrorHandling::Silent:
					default:
						// Do nothing
						break;
				}
				break;
			}
			default:
			{
				// We expect all exit situations to have an explicit case.
				check(false);
				break;
			}
		}

		TArray< FAnalyticsEventAttribute > EventAttributes;
		EventAttributes.Emplace( TEXT("State"), (int32)AuthorizationState );
		EventAttributes.Emplace( TEXT("PreviousState"), (int32)PreviousAuthorizationState );
		EventAttributes.Emplace( TEXT("UnauthorizedErrorHandling"), (int32)UnauthorizedErrorHandling );
		EventAttributes.Emplace( TEXT("ItemId"), Authorizer->GetPluginItemId() );
		EventAttributes.Emplace( TEXT("OfferId"), Authorizer->GetPluginOfferId() );

		FEngineAnalytics::GetProvider().RecordEvent( TEXT("PluginWarden.AuthorizationFailure"), EventAttributes );
	}
}

void SAuthorizingPlugin::ShowStorePageForPlugin()
{
	ILauncherPlatform* LauncherPlatform = FLauncherPlatformModule::Get();

	if (LauncherPlatform != nullptr )
	{
		FOpenLauncherOptions StorePageOpen(FString(TEXT("/ue/marketplace/content/")) + Authorizer->GetPluginOfferId());
		LauncherPlatform->OpenLauncher(StorePageOpen);
	}
}

#undef LOCTEXT_NAMESPACE
