// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Async/AsyncResult.h"
#include "IPluginWardenModule.h"
#include "PluginWardenAuthorizer.h"

enum class EEntitlementCacheLevelRequest : uint8;
class FPluginWardenAuthorizer;

extern TSet<FString> AuthorizedPlugins;

/**
 * The authorizing plug-in ui guides the user through the process of certifying their access to the plug-in.
 */
class SAuthorizingPlugin : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAuthorizingPlugin){}

	SLATE_END_ARGS()

	struct FPluginInfo
	{
		FPluginInfo( const FText& InFriendlyName, const FString& InItemId, const FString& InOfferId )
			: FriendlyName( InFriendlyName )
			, ItemId( InItemId )
			, OfferId( InOfferId )
		{
		}

		const FText& FriendlyName;
		const FString& ItemId;
		const FString& OfferId;
	};

	void Construct(const FArguments& InArgs, const TSharedRef<SWindow>& InParentWindow, const FPluginInfo& PluginInfo, const EEntitlementCacheLevelRequest CacheLevel, TFunction<void()> InAuthorizedCallback);

	/**
	 * Override the default message and behavior in the case where the plugin is unauthorized.
	 *
	 * @param UnauthorizedMessageOverride The error message to display for unauthorized plugins, overriding the default message if not empty.
	 * @param UnauthorizedErrorHandling How to handle the unauthorized error.
	 */
	void SetUnauthorizedOverride(const FText& InUnauthorizedMessageOverride, IPluginWardenModule::EUnauthorizedErrorHandling UnauthorizedErrorHandling);

private:

	FText GetWaitingText() const;

	EActiveTimerReturnType RefreshStatus(double InCurrentTime, float InDeltaTime);

	/** Called when the user presses the Cancel button. */
	FReply OnCancel();

	void OnWindowClosed(const TSharedRef<SWindow>& InWindow);

	/** Show the store page for the plug-in, happens in response to the user asking to see the store page when license detection fails. */
	void ShowStorePageForPlugin();

private:
	/** The parent window holding this dialog, for when we need to trigger a close. */
	TWeakPtr<SWindow> ParentWindow;

	/** The optional error message to display in case plugin is unauthorized. If empty, will default to standard message. */
	FText UnauthorizedMessageOverride;

	/** Flag that sets the handling mechanism for when plugin is not authorized */
	IPluginWardenModule::EUnauthorizedErrorHandling UnauthorizedErrorHandling;

	/** Flag for tracking user interruption of the process, either with the cancel button or the close button. */
	bool bUserInterrupted;

	/** The latest state of the plug-in authorization pipeline. */
	EPluginAuthorizationState AuthorizationState;

	/** The previous state of the plug-in authorization pipeline. */
	EPluginAuthorizationState PreviousAuthorizationState;

	/** If the user is authorized to us the plug-in, we'll call this function to alert the plug-in that everything is good to go. */
	TFunction<void()> AuthorizedCallback;

	/** The executioner of the authorization pipeline. */
	TSharedPtr<FPluginWardenAuthorizer> Authorizer;
};
