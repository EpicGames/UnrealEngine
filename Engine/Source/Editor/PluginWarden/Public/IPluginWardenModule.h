// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

enum class EEntitlementCacheLevelRequest : uint8;

/**
 * The Plugin Warden is a simple module used to verify a user has purchased a plug-in.  This
 * module won't prevent a determined user from avoiding paying for a plug-in, it is merely to
 * prevent accidental violation of a per-seat license on a plug-in, and to direct those users
 * to the marketplace page where they may purchase the plug-in.
 */
class IPluginWardenModule : public IModuleInterface
{
public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPluginWardenModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IPluginWardenModule >( "PluginWarden" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "PluginWarden" );
	}

	/**
	 * This enum efines how Unauthorized items are handled
	 */
	enum class EUnauthorizedErrorHandling
	{
		/** Handle error silently, no popups */
		Silent,
		/** Show the default or overriden message in a popup */
		ShowMessage,
		/** Show the default or overriden message in a popup, and ask the user if they want to open the Marketplace */
		ShowMessageOpenStore
	};

	/**
	 * Ask the Unreal Engine Launcher if the user has authorization to use the given plug-in. The authorized 
	 * callback will only be called if the user is authorized to use the plug-in.
	 *
	 * ### WARNING ### WARNING ### WARNING ### WARNING ### WARNING ###
	 *
	 * Do not gate the user in inline custom plug-in UI, like inside a customization in the details panel.  Only use 
	 * this to gate the user from opening a dialog or some other big explicit action that opens up into UI that is 
	 * exclusively the domain of your plug-in.  An example of a good place to use this would be inside of  
	 * OpenAssetEditor(), in your derived version of FAssetTypeActions_Base for the custom assets your plug-in handles.
	 *
	 * ### WARNING ### WARNING ### WARNING ### WARNING ### WARNING ###
	 *
	 * IPluginWardenModule::Get().CheckEntitlementForPlugin(LOCTEXT("AwesomePluginName", "My Awesome Plugin"), TEXT("PLUGIN_MARKETPLACE_GUID"), [&] () {
	 *		// Code Here Will Run If Authorized
	 * });
	 * 
	 * @param PluginFriendlyName The localized friendly name of the plug-in.
	 * @param PluginItemId The unique identifier of the item plug-in on the marketplace.
	 * @param PluginOfferId The unique identifier of the offer for the plug-in on the marketplace.
	 * @param CacheLevel Where to check for the entitlements. Defaults to memory.
	 * @param UnauthorizedMessageOverride The error message to display for unauthorized plugins, overriding the default message if not empty.
	 * @param UnauthorizedErrorHandling How to handle the unauthorized error.
	 * @param AuthorizedCallback This function will be called after the user has been given entitlement.
	 */
	virtual void CheckEntitlementForPlugin(const FText& PluginFriendlyName, const FString& PluginItemId, const FString& PluginOfferId, const EEntitlementCacheLevelRequest CacheLevel,
		const FText& UnauthorizedMessageOverride, EUnauthorizedErrorHandling UnauthorizedErrorHandling, TFunction<void()> AuthorizedCallback) = 0;

	virtual void CheckEntitlementForPlugin(const FText& PluginFriendlyName, const FString& PluginItemId, const FString& PluginOfferId,
		const FText& UnauthorizedMessageOverride, EUnauthorizedErrorHandling UnauthorizedErrorHandling, TFunction<void()> AuthorizedCallback) = 0;
};
